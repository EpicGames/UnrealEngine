{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ICVFXTesting",
	"Description": "Testing utilities to be used by ICVFX projects",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Hidden": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "Gauntlet",
			"Enabled": true
		},
		{
			"Name": "LiveLinkCamera",
			"Enabled": true
		},
		{
			"Name": "nDisplay",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "ICVFXTesting",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	]
}
