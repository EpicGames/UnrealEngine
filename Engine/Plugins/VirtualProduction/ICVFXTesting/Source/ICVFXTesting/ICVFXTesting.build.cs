// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ICVFXTesting : ModuleRules
{
	public ICVFXTesting(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"CoreUObject",
				"Gauntlet"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
            	"DisplayCluster",
				"Engine",
                "LiveLink",
                "LiveLinkComponents",
				"LiveLinkInterface",
				"RenderCore"
			}
        );
	}
}
