// Copyright Epic Games, Inc. All Rights Reserved.

#include "ICVFXTestControllerAutoTest.h"

#include "CineCameraActor.h"
#include "Camera/CameraComponent.h"
#include "DisplayClusterRootActor.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "EngineUtils.h"
#include "GameFramework/GameMode.h"
#include "IDisplayCluster.h"
#include "ICVFXTestLocation.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkRole.h"
#include "LiveLinkVirtualSubject.h"
#include "Logging/LogVerbosity.h"
#include "PlatformFeatures.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "VideoRecordingSystem.h"
#include "Roles/LiveLinkTransformRole.h"
#include "UObject/SoftObjectPtr.h"


CSV_DEFINE_CATEGORY(ICVFXTest, true);

namespace ICVFXTest
{
	static TAutoConsoleVariable<float> CVarSoakTime(
		TEXT("ICVFXTest.SoakTime"),
		30.0f,
		TEXT("Duration of the sandbox soak, in seconds, per run."),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarSkipTestSequence(
		TEXT("ICVFXTest.SkipTestSequence"),
		false,
		TEXT("Whether to skip the test sequence after the sandbox soak, if applicable."),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarRequestShutdown(
		TEXT("ICVFXTest.RequestShutdown"),
		false,
		TEXT("Whether to request shutdown when the current run is finished, regardless of NumRuns."),
		ECVF_Default);

	static TAutoConsoleVariable<FString> CVarTraceFileName(
		TEXT("ICVFXTest.TraceFileName"),
		"",
		TEXT("Path to the output tracefile."),
		ECVF_Default);
}

//////////////////////////////////////////////////////////////////////
// States
//////////////////////////////////////////////////////////////////////

// Base
void FICVFXAutoTestState::Start(const EICVFXAutoTestState PrevState)
{
	check(IsValid(Controller));
	TimeSinceStart = 0.0;
}

void FICVFXAutoTestState::Tick(const float TimeDelta)
{
	TimeSinceStart += TimeDelta;
}

// InitialLoad
class FICVFXAutoTestState_InitialLoad : public FICVFXAutoTestState
{
public:
	FICVFXAutoTestState_InitialLoad(UICVFXTestControllerAutoTest* const TestController) 
		: FICVFXAutoTestState(TestController) 
	{
		ENQUEUE_RENDER_COMMAND(SetInnerGPUIndex)(
			[TestController](FRHICommandListImmediate& RHICmdList)
			{
#if WITH_MGPU
				TestController->SetInnerGPUIndex(RHICmdList.GetGPUMask().GetLastIndex());
#endif
			}
		);
	}

	virtual void Start(const EICVFXAutoTestState PrevState) override
	{
		
	}

	virtual void Tick(float TimeDelta) override
	{
		Super::Tick(TimeDelta);

		if (GetTestStateTime() >= MaxStateChangeWait)
		{
			UE_LOGF(LogICVFXTest, Warning, "AutoTest InitialLoad %ls: waited for state change for %f seconds, bindings may be missing, validate current state.", ANSI_TO_TCHAR(__func__), MaxStateChangeWait);
			Controller->SetTestState(EICVFXAutoTestState::Soak);
		}
	}

private:
	const float MaxStateChangeWait = 30.0f;
};

// Sandbox Soak
class FICVFXAutoTestState_Soak : public FICVFXAutoTestState
{
public:
	FICVFXAutoTestState_Soak(UICVFXTestControllerAutoTest* const TestController) : FICVFXAutoTestState(TestController) {}

	virtual void Start(const EICVFXAutoTestState PrevState) override
	{
		Super::Start(PrevState);

		// If starting a new run, kick off requested captures
		if (PrevState == EICVFXAutoTestState::InitialLoad || PrevState == EICVFXAutoTestState::Finished)
		{
			if (Controller->RequestsVideoCapture())
			{
				Controller->TryStartingVideoCapture();
			}

			if (Controller->RequestsFPSChart())
			{
				Controller->ConsoleCommand(TEXT("StartFPSChart"));
			}

			// Try finding the display cluster root actor to move it during the next phases.
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartScene().AddLambda([this]()
			{
				Controller->DisplayClusterActor = IDisplayCluster::Get().GetGameMgr()->GetRootActor();
				Controller->UpdateInnerGPUIndex();
				Controller->InitializeLiveLink();
				Controller->UpdateTestLocations();
			});
		}

		const float SoakTime = ICVFXTest::CVarSoakTime->GetFloat();
		if (SoakTime > 0.0f)
		{
			UE_LOGF(LogICVFXTest, Display, "AutoTest Soak %ls: soak time started for %f seconds...", ANSI_TO_TCHAR(__func__), SoakTime);
		}
		else
		{
			UE_LOGF(LogICVFXTest, Display, "AutoTest Soak %ls: soaking indefinitely until soak time is set >0...", ANSI_TO_TCHAR(__func__));
		}
	}

	virtual void Tick(float TimeDelta) override
	{
		Super::Tick(TimeDelta);
		
		const float SoakTime = ICVFXTest::CVarSoakTime->GetFloat();
		if (SoakTime > 0.0f && GetTestStateTime() >= SoakTime)
		{
			if (Controller->NumTestLocations())
			{
				Controller->SetTestState(EICVFXAutoTestState::TraverseTestLocations);
			}
			else
			{
				Controller->SetTestState(EICVFXAutoTestState::Finished);
			}
		}		
	}
};

// Go through test locations and collect perf data.
class FICVFXAutoTestState_TraverseTestLocations : public FICVFXAutoTestState
{
public:
	FICVFXAutoTestState_TraverseTestLocations(UICVFXTestControllerAutoTest* const TestController) : FICVFXAutoTestState(TestController) {}

	virtual void Start(const EICVFXAutoTestState PrevState) override
	{
		Super::Start(PrevState);

		const FString TraceFile = ICVFXTest::CVarTraceFileName.GetValueOnAnyThread();
		if (!TraceFile.IsEmpty())
		{
			Controller->ConsoleCommand(*(FString(TEXT("trace.file ")) + ICVFXTest::CVarTraceFileName.GetValueOnAnyThread()));
		}

		Controller->GoToTestLocation(0);

		UE_LOGF(LogICVFXTest, Display, "AutoTest TraverseTestLocations %ls: started for %f seconds...", ANSI_TO_TCHAR(__func__), Controller->TimePerTestLocation)
	}

	virtual void Tick(float TimeDelta) override
	{
		Super::Tick(TimeDelta);

		Controller->TimeAtTestLocation += TimeDelta;
		
		if (GetTestStateTime() >= Controller->TimePerTestLocation * Controller->NumTestLocations())
		{
			Controller->SetTestState(EICVFXAutoTestState::Finished);
		}
		else if (Controller->TimeAtTestLocation >= Controller->TimePerTestLocation)
		{
			if (Controller->GetCurrentTestLocationIndex() == Controller->NumTestLocations() - 1)
			{
				Controller->SetTestState(EICVFXAutoTestState::Finished);
			}
			else
			{
				Controller->GoToNextTestLocation();
			}
		}
	}
};

// Finished
class FICVFXAutoTestState_Finished : public FICVFXAutoTestState
{
public:
	FICVFXAutoTestState_Finished(UICVFXTestControllerAutoTest* const TestController) : FICVFXAutoTestState(TestController) {}

	virtual void Start(const EICVFXAutoTestState PrevState) override
	{
		Super::Start(PrevState);
		
		if (!ICVFXTest::CVarTraceFileName.GetValueOnAnyThread().IsEmpty())
		{
			Controller->ConsoleCommand(TEXT("trace.stop"));
		}

		if (Controller->RequestsMemReport())
		{
			Controller->ExecuteMemReport();
		}

		Controller->MarkRunComplete();

		// Success validation goes here...

		bWaitingOnFPSChart = false;
		bWaitingOnFinalizingVideo = false;

#if CSV_PROFILER_STATS
		if (FCsvProfiler* const CsvProfiler = FCsvProfiler::Get())
		{
			CsvProfilerDelegateHandle = CsvProfiler->OnCSVProfileFinished().AddLambda([this](const FString& Filename)
			{
				UE_LOGF(LogICVFXTest, Display, "AutoTest %ls: OnCSVProfileFinished: CsvProfiler finished...", ANSI_TO_TCHAR(__func__));
				bWaitingOnFPSChart = false;
			});

			if (CsvProfiler->IsCapturing())
			{
				bWaitingOnFPSChart = true;
				Controller->ConsoleCommand(TEXT("StopFPSChart"));
			}
			else
			{
				bWaitingOnFPSChart = CsvProfiler->IsWritingFile();
			}

		}
#endif // CSV_PROFILER_STATS

		if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
		{
			VideoRecordingDelegateHandle = VideoRecordingSystem->GetOnVideoRecordingFinalizedDelegate().AddLambda([this](bool bSucceeded, const FString& FilePath)
			{
				if (bSucceeded)
				{
					UE_LOGF(LogICVFXTest, Display, "AutoTest OnVideoRecordingFinalized: video recording successfully finalized at %ls.", *FilePath);
				}
				else
				{
					UE_LOGF(LogICVFXTest, Warning, "AutoTest OnVideoRecordingFinalized: video recording failed to finalize...");
				}

				bWaitingOnFinalizingVideo = false;
			});
		}

		bWaitingOnFinalizingVideo = Controller->TryFinalizingVideoCapture(true);
	}

	virtual void Tick(const float TimeDelta) override
	{
		Super::Tick(TimeDelta);

		// Checks whether the test shutdown is complete and the test is ready to end
		if (!bWaitingOnFPSChart && !bWaitingOnFinalizingVideo)
		{
			if (Controller->GetRunsRemaining() > 0 && !ICVFXTest::CVarRequestShutdown->GetBool())
			{
				const UWorld* const World = Controller->GetWorld();
				if (AGameMode* const GameMode = World ? World->GetAuthGameMode<AGameMode>() : nullptr)
				{
					GameMode->RestartGame();
				}
			}
			else
			{
				// If all runs are complete, then cleanup and shutdown test
				Controller->SetTestState(EICVFXAutoTestState::Shutdown);
			}
		}
	}

	virtual void End(const EICVFXAutoTestState NewState) override
	{
#if CSV_PROFILER_STATS
		if (FCsvProfiler* const CsvProfiler = FCsvProfiler::Get())
		{
			CsvProfiler->OnCSVProfileFinished().Remove(CsvProfilerDelegateHandle);
		}
#endif // CSV_PROFILER_STATS

		if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
		{
			VideoRecordingSystem->GetOnVideoRecordingFinalizedDelegate().Remove(VideoRecordingDelegateHandle);
		}

		Super::End(NewState);
	}

private:
	bool bWaitingOnFPSChart = false;
	bool bWaitingOnFinalizingVideo = false;

	FDelegateHandle CsvProfilerDelegateHandle;
	FDelegateHandle VideoRecordingDelegateHandle;
};

// Finished
class FICVFXAutoTestState_Shutdown : public FICVFXAutoTestState
{
public:
	FICVFXAutoTestState_Shutdown(UICVFXTestControllerAutoTest* const TestController) : FICVFXAutoTestState(TestController) {}

	virtual void Start(const EICVFXAutoTestState PrevState) override
	{
		Super::Start(PrevState);
		Controller->EndICVFXTest(TestExitCode);
	}

	void SetTestExitCode(const int32 ExitCode)
	{
		TestExitCode = ExitCode;
	}

private:
	int32 TestExitCode = 0;
};


//////////////////////////////////////////////////////////////////////
// Non state-specific logic
//////////////////////////////////////////////////////////////////////


FString UICVFXTestControllerAutoTest::GetStateName(const EICVFXAutoTestState State) const
{
	return StaticEnum<EICVFXAutoTestState>()->GetValueAsString(State);
}

FICVFXAutoTestState& UICVFXTestControllerAutoTest::GetTestState() const
{
	check(CurrentState != EICVFXAutoTestState::MAX);
	FICVFXAutoTestState* State = States[(uint8)CurrentState];
	check(State);
	return *State;
}

FICVFXAutoTestState& UICVFXTestControllerAutoTest::SetTestState(const EICVFXAutoTestState NewState)
{
	const FString& EndingStateStr = GetStateName(CurrentState);
	const FString& StartingStateStr = GetStateName(NewState);

	if (CurrentState == NewState)
	{
		UE_LOGF(LogICVFXTest, Display, "AutoTest %ls: reloading %ls", ANSI_TO_TCHAR(__func__), *EndingStateStr);
	}
	else
	{
		UE_LOGF(LogICVFXTest, Display, "AutoTest %ls: %ls -> %ls", ANSI_TO_TCHAR(__func__), *EndingStateStr, *StartingStateStr);
	}

	// End current test state
	GetTestState().End(NewState);

	// Transition current test state from the previous state to the new state
	const EICVFXAutoTestState PrevState = CurrentState;
	CurrentState = NewState;
	FICVFXAutoTestState& CurrentTestState = GetTestState();

	// Start new test state
	CurrentTestState.Start(PrevState);
	return CurrentTestState;
}

void UICVFXTestControllerAutoTest::GoToTestLocation(int32 Index)
{
	FString TestLocationName = TestLocations[Index]->GetActorNameOrLabel();

	CSV_EVENT(ICVFXTest, TEXT("TestLocation %s"), *TestLocationName);

	TimeAtTestLocation = 0.0;

	UE_LOGF(LogICVFXTest, Display, "AutoTest TraverseTestLocations: Moving to test location: %ls", *TestLocations[Index]->GetActorNameOrLabel());
	DisplayClusterActor->SetActorTransform(TestLocations[Index]->GetActorTransform());
}

void UICVFXTestControllerAutoTest::SetTestLocations(const TArray<AActor*> InTestLocations)
{
	TestLocations = InTestLocations;
	StateTimeouts[(uint8)EICVFXAutoTestState::TraverseTestLocations] = TestLocations.Num() * TimePerTestLocation;
}

void UICVFXTestControllerAutoTest::InitializeLiveLink()
{
	if (!IsValid(DisplayClusterActor))
	{
		return;
	}

	// LL preset is located in the same (ICVFXTesting) plugin, therefore must always exist
	ULiveLinkPreset* LLPreset = LoadObject<ULiveLinkPreset>(NULL, TEXT("/ICVFXTesting/AutomationPreset"));
	if (!IsValid(LLPreset))
	{
		check(false);
		return;
	}

	UDisplayClusterICVFXCameraComponent* const Component = DisplayClusterActor->FindComponentByClass<UDisplayClusterICVFXCameraComponent>();
	if (!IsValid(Component))
	{
		return;
	}

	const FDisplayClusterConfigurationICVFX_CameraSettings& CameraSettings = Component->GetCameraSettingsICVFX();
	ACineCameraActor* const CameraActor = CameraSettings.ExternalCameraActor.Get();
	if (!IsValid(CameraActor))
	{
		return;
	}

	ULiveLinkComponentController* LiveLinkComponentController = NewObject<ULiveLinkComponentController>(CameraActor);
	LiveLinkComponentController->bUpdateInEditor = true;
	LiveLinkComponentController->bEvaluateLiveLink = true;

	CameraActor->AddInstanceComponent(LiveLinkComponentController);
	LiveLinkComponentController->RegisterComponentWithWorld(CameraActor->GetWorld());

	FName LLVirtualSubjectName;
	if (LLPreset->GetSubjectPresets().Num())
	{
		LLVirtualSubjectName = LLPreset->GetSubjectPresets()[0].Key.SubjectName.Name;
	}

	LLPreset->ApplyToClientLatent([LiveLinkComponentController, LLVirtualSubjectName, CameraActor](bool bSuccess)
		{
			if (!IsValid(LiveLinkComponentController) || !IsValid(CameraActor))
			{
				return;
			}

			if (bSuccess)
			{
				FLiveLinkSubjectRepresentation SubjectRepresentation;
				SubjectRepresentation.Role = ULiveLinkTransformRole::StaticClass();
				SubjectRepresentation.Subject = LLVirtualSubjectName;

				LiveLinkComponentController->SetSubjectRepresentation(SubjectRepresentation);
				LiveLinkComponentController->SetControlledComponent(ULiveLinkTransformRole::StaticClass(), CameraActor->GetCineCameraComponent());

				if (TObjectPtr<ULiveLinkControllerBase>* Controller = LiveLinkComponentController->ControllerMap.Find(ULiveLinkTransformRole::StaticClass()))
				{
					if (ULiveLinkTransformController* TransformController = Cast<ULiveLinkTransformController>(*Controller))
					{
						TransformController->TransformData.bUseLocation = false;
					}
				}
			}
		});
}

void UICVFXTestControllerAutoTest::UpdateInnerGPUIndex()
{
	TArray<UDisplayClusterICVFXCameraComponent*> CameraComponents;
	ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(DisplayClusterActor);
	if (!IsValid(RootActor))
	{
		return;
	}

	constexpr bool bIncludeChildActors = false;
	RootActor->GetComponents<UDisplayClusterICVFXCameraComponent>(CameraComponents, bIncludeChildActors);

	UWorld* CurrentWorld = RootActor->GetWorld();

	for (UDisplayClusterICVFXCameraComponent* CameraComponent : CameraComponents)
	{
		CameraComponent->CameraSettings.RenderSettings.AdvancedRenderSettings.GPUIndex = GetInnerGPUIndex();
	}

	if (IDisplayClusterViewportManager* ViewportManager = RootActor->GetViewportManager())
	{
		const FString NodeId = IDisplayCluster::Get().GetClusterMgr()->GetNodeId();
		const EDisplayClusterRenderFrameMode RenderMode = EDisplayClusterRenderFrameMode::Mono;

		ViewportManager->GetConfiguration().UpdateConfigurationForClusterNode(RenderMode, CurrentWorld, NodeId);
	}
}

void UICVFXTestControllerAutoTest::UpdateTestLocations()
{
	UClass* TestLocationClass = AICVFXTestLocation::StaticClass();

	TestLocations.Reset();

	for (TActorIterator<AActor> ItActor = TActorIterator<AActor>(GetWorld(), TestLocationClass); ItActor; ++ItActor)
	{
		TestLocations.Add(*ItActor);
	}

	if (TestLocations.Num())
	{
		UE_LOGF(LogICVFXTest, Display, "Found %d test locations.", TestLocations.Num());
	}
	else
	{
		UE_LOGF(LogICVFXTest, Display, "Could not find test locations, defaulting to display cluster position.");
		// Todo: If no uobject path passed to us, default on the first display cluster actor we find?
		TestLocations.Add(DisplayClusterActor);
	}

	SetTestLocations(TestLocations);
}

void UICVFXTestControllerAutoTest::EndICVFXTest(const int32 ExitCode /*= 0*/)
{
	if (CurrentState != EICVFXAutoTestState::Shutdown)
	{
		UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: called outside Shutdown test state, something went wrong.", ANSI_TO_TCHAR(__func__));
		static_cast<FICVFXAutoTestState_Shutdown&>(SetTestState(EICVFXAutoTestState::Shutdown)).SetTestExitCode(ExitCode);
		return;
	}

	Super::EndICVFXTest(ExitCode);
}

void UICVFXTestControllerAutoTest::UnbindAllDelegates()
{
	if (UWorld* const World = GetWorld())
	{
		World->OnWorldBeginPlay.RemoveAll(this);
		World->GameStateSetEvent.RemoveAll(this);
	}

	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(SoakTimeSink);

	Super::UnbindAllDelegates();
}

void UICVFXTestControllerAutoTest::OnInit()
{
	Super::OnInit();

	// Initialize State Data
	States[(uint8)EICVFXAutoTestState::InitialLoad] = new FICVFXAutoTestState_InitialLoad(this);
	States[(uint8)EICVFXAutoTestState::Soak] = new FICVFXAutoTestState_Soak(this);
	States[(uint8)EICVFXAutoTestState::TraverseTestLocations] = new FICVFXAutoTestState_TraverseTestLocations(this);
	States[(uint8)EICVFXAutoTestState::Finished] = new FICVFXAutoTestState_Finished(this);
	States[(uint8)EICVFXAutoTestState::Shutdown] = new FICVFXAutoTestState_Shutdown(this);

	StateTimeouts[(uint8)EICVFXAutoTestState::InitialLoad] = 60.f;
	StateTimeouts[(uint8)EICVFXAutoTestState::Soak] =  ICVFXTest::CVarSoakTime->GetFloat();
	StateTimeouts[(uint8)EICVFXAutoTestState::TraverseTestLocations] = 60.0f;
	StateTimeouts[(uint8)EICVFXAutoTestState::Finished] = 300.f;
	StateTimeouts[(uint8)EICVFXAutoTestState::Shutdown] = 30.f;

	for (int32 StateIdx = 0; StateIdx < (uint8)EICVFXAutoTestState::MAX; ++StateIdx)
	{
		checkf(States[StateIdx] != nullptr, TEXT("Missing state object for state %s"), *GetStateName((EICVFXAutoTestState)StateIdx));
	}

	const FConsoleCommandDelegate SoakTimeDelegate = FConsoleCommandDelegate::CreateUObject(this, &ThisClass::OnSoakTimeChanged);
	SoakTimeSink = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(SoakTimeDelegate);

	// Set and start the InitialLoad test state
	UE_LOGF(LogICVFXTest, Display, "AutoTest %ls: setting test state to InitialLoad", ANSI_TO_TCHAR(__func__));
	CurrentState = EICVFXAutoTestState::InitialLoad;
	GetTestState().Start(EICVFXAutoTestState::MAX);
}

void UICVFXTestControllerAutoTest::OnPreMapChange()
{
	Super::OnPreMapChange();

	if (CurrentState != EICVFXAutoTestState::InitialLoad)
	{
		UE_LOGF(LogICVFXTest, Display, "AutoTest %ls: setting test state to InitialLoad", ANSI_TO_TCHAR(__func__));
		SetTestState(EICVFXAutoTestState::InitialLoad);
	}
}

void UICVFXTestControllerAutoTest::OnTick(float TimeDelta)
{
	Super::OnTick(TimeDelta);

	GetTestState().Tick(TimeDelta);
	MarkHeartbeatActive();

	if (GetTestState().GetTestStateTime() > StateTimeouts[(uint8)CurrentState])
	{
		if (CurrentState == EICVFXAutoTestState::Finished || CurrentState == EICVFXAutoTestState::Shutdown)
		{
			// Treat timeouts during finalization/shutdown as warnings
			UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: '%ls' test state timed out after %f seconds", ANSI_TO_TCHAR(__func__), *GetStateName(CurrentState), StateTimeouts[(uint8)CurrentState]);
			EndICVFXTest();
		}
		else
		{
			// Treat all other timeouts as errors
			UE_LOGF(LogICVFXTest, Error, "AutoTest %ls: '%ls' test state timed out after %f seconds", ANSI_TO_TCHAR(__func__), *GetStateName(CurrentState), StateTimeouts[(uint8)CurrentState]);
			EndICVFXTest(1);
		}
	}
}

void UICVFXTestControllerAutoTest::BeginDestroy()
{
	UnbindAllDelegates();

	for (int32 StateIdx = 0; StateIdx < (uint8)EICVFXAutoTestState::MAX; ++StateIdx)
	{
		delete States[StateIdx];
		States[StateIdx] = nullptr;
	}

	Super::BeginDestroy();
}

void UICVFXTestControllerAutoTest::OnPreWorldInitialize(UWorld* const World)
{
	check(World);
	World->GameStateSetEvent.AddUObject(this, &ThisClass::OnGameStateSet);
	World->OnWorldBeginPlay.AddUObject(this, &ThisClass::OnWorldBeginPlay);
}

void UICVFXTestControllerAutoTest::OnGameStateSet(AGameStateBase* const GameStateBase)
{
	if (UWorld* const World = GetWorld())
	{
		World->GameStateSetEvent.RemoveAll(this);
	}
}

void UICVFXTestControllerAutoTest::OnWorldBeginPlay()
{
	SetTestState(EICVFXAutoTestState::Soak);
	
	if (RequestsMemReport())
	{
		ExecuteMemReport();
		SetMemReportTimer();
	}
}

void UICVFXTestControllerAutoTest::OnSoakTimeChanged()
{
	const float SoakTime = ICVFXTest::CVarSoakTime->GetFloat();
	StateTimeouts[(uint8)EICVFXAutoTestState::Soak] = SoakTime > 0.0f ? SoakTime + 15.0f : TNumericLimits<float>::Max();

	if (CurrentState == EICVFXAutoTestState::Soak)
	{
		UE_LOGF(LogICVFXTest, Display, "AutoTest Soak %ls: soak time changed to %f seconds...", ANSI_TO_TCHAR(__func__), ICVFXTest::CVarSoakTime->GetFloat());
	}
}
