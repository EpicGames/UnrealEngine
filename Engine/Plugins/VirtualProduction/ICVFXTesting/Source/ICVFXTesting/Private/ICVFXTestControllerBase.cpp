// Copyright Epic Games, Inc. All Rights Reserved.

#include "ICVFXTestControllerBase.h"

#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/EngineVersion.h"
#include "PlatformFeatures.h"
#include "VideoRecordingSystem.h"

#include "GauntletModule.h"

DEFINE_LOG_CATEGORY(LogICVFXTest);

namespace ICVFXTest
{
	static TAutoConsoleVariable<int32> CVarMaxRunCount(
		TEXT("ICVFXTest.MaxRunCount"),
		1,
		TEXT("Max number of runs desired for this session."),
		ECVF_Default);

	static TAutoConsoleVariable<FString> CVarMemReportArgs(
		TEXT("ICVFXTest.MemReportArgs"),
		TEXT("-full -csv"),
		TEXT("Arguments passed to the MemReport console command."),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarMemReportInterval(
		TEXT("ICVFXTest.MemReportInterval"),
		60.0f * 30.0f,
		TEXT("Duration in seconds between MemReport calls during automated testing."),
		ECVF_Default);
}


UICVFXTestControllerBase::UICVFXTestControllerBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WarnStuckTime(10 * 60)
	, NextWarnStuckTime(0)
	, RunCount(0)
	, bRequestsFPSChart(false)
	, bRequestsMemReport(false)
	, bRequestsVideoCapture(false)
{
}


void UICVFXTestControllerBase::OnInit()
{
	Super::OnInit();

	int32 MaxRunCount;
	if (FParse::Value(FCommandLine::Get(), TEXT("ICVFXTest.MaxRunCount"), MaxRunCount))
	{
		ICVFXTest::CVarMaxRunCount->Set(MaxRunCount);
	}

	bRequestsFPSChart = FParse::Param(FCommandLine::Get(), TEXT("ICVFXTest.FPSChart"));
	bRequestsMemReport = FParse::Param(FCommandLine::Get(), TEXT("ICVFXTest.MemReport"));
	bRequestsVideoCapture = FParse::Param(FCommandLine::Get(), TEXT("ICVFXTest.VideoCapture"));

	FParse::Value(FCommandLine::Get(), TEXT("ICVFXTest.DisplayClusterUAssetPath"), DisplayClusterUObjectPath);

	DisplayClusterUObjectPath.RightChopInline(1);
	UE_LOGF(LogTemp, Error, "Test jere-1: %ls", *DisplayClusterUObjectPath);

	MemReportTimerDelegate.BindUObject(this, &ThisClass::OnMemReportTimerExpired);

	const FConsoleCommandDelegate MemReportIntervalDelegate = FConsoleCommandDelegate::CreateUObject(this, &ThisClass::OnMemReportIntervalChanged);
	MemReportIntervalSink = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(MemReportIntervalDelegate);

	FWorldDelegates::OnPreWorldInitialization.AddUObject(this, &ThisClass::OnPreWorldInitializeInternal);
}

void UICVFXTestControllerBase::OnTick(float TimeDelta)
{
	if ((NextWarnStuckTime > 0) && (GetTimeInCurrentState() >= NextWarnStuckTime))
	{
		UE_LOGF(LogICVFXTest, Display, "Have been in same state for %.02f mins", GetTimeInCurrentState() / 60.0f);
		NextWarnStuckTime += WarnStuckTime;
	}
}

void UICVFXTestControllerBase::OnStateChange(FName OldState, FName NewState)
{
	if ((NewState != GetCurrentState()) && (NewState != FGauntletStates::Initialized))
	{
		NextWarnStuckTime = WarnStuckTime;
	}
}

void UICVFXTestControllerBase::BeginDestroy()
{
	UnbindAllDelegates();
	Super::BeginDestroy();
}

void UICVFXTestControllerBase::EndICVFXTest(const int32 ExitCode/*=0*/)
{
	UnbindAllDelegates();

	UE_LOGF(LogICVFXTest, Display, "%ls: test completed, requesting exit...", ANSI_TO_TCHAR(__func__));
	EndTest(ExitCode);
}

void UICVFXTestControllerBase::UnbindAllDelegates()
{
	ClearMemReportTimer();
	MemReportTimerDelegate.Unbind();
	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(MemReportIntervalSink);
	FWorldDelegates::OnPreWorldInitialization.RemoveAll(this);
}

uint32 UICVFXTestControllerBase::GetRunCount() const
{
	return RunCount;
}

uint32 UICVFXTestControllerBase::GetMaxRunCount() const
{
	const int32 MaxRunCount = ICVFXTest::CVarMaxRunCount->GetInt();
	return MaxRunCount > 0 ? MaxRunCount : TNumericLimits<uint32>::Max();
}

uint32 UICVFXTestControllerBase::GetRunsRemaining() const
{
	return GetMaxRunCount() - GetRunCount();
}

uint32 UICVFXTestControllerBase::MarkRunComplete()
{
	++RunCount;
	UE_LOGF(LogICVFXTest, Display, "%ls: run complete (%u/%u)...", ANSI_TO_TCHAR(__func__), GetRunCount(), GetMaxRunCount());
	return RunCount;
}

bool UICVFXTestControllerBase::RequestsFPSChart()
{
	return bRequestsFPSChart;
}

bool UICVFXTestControllerBase::RequestsMemReport()
{
	return bRequestsMemReport;
}

bool UICVFXTestControllerBase::RequestsVideoCapture()
{
	return bRequestsVideoCapture;
}

void UICVFXTestControllerBase::ConsoleCommand(const TCHAR* Cmd)
{
	if (APlayerController* Controller = GetFirstPlayerController())
	{
		Controller->ConsoleCommand(Cmd);
		UE_LOGF(LogICVFXTest, Display, "Issued console command: '%ls'", Cmd);
	}
}

void UICVFXTestControllerBase::ExecuteMemReport(const TOptional<FString> Args/*=TOptional<FString>()*/)
{
	ConsoleCommand(*FString(TEXT("MemReport ") + (Args.IsSet() ? Args.GetValue() : ICVFXTest::CVarMemReportArgs->GetString())));
}

void UICVFXTestControllerBase::SetMemReportTimer(const TOptional<float> Interval/*=TOptional<float>()*/)
{
	if (UWorld* const World = GetWorld())
	{
		FTimerManager& TimerManager = World->GetTimerManager();

		const float AdjustedInterval = Interval.IsSet() ? Interval.GetValue() : ICVFXTest::CVarMemReportInterval->GetFloat();
		if (AdjustedInterval > 0.0f)
		{
			UE_LOGF(LogICVFXTest, Display, "%ls: interval is set to %f seconds...", ANSI_TO_TCHAR(__func__), AdjustedInterval);
			TimerManager.SetTimer(MemReportTimerHandle, MemReportTimerDelegate, AdjustedInterval, true);
		}
		else if (MemReportTimerHandle.IsValid())
		{
			UE_LOGF(LogICVFXTest, Display, "%ls: interval is set to 0, pausing timer...", ANSI_TO_TCHAR(__func__));
			TimerManager.PauseTimer(MemReportTimerHandle);
		}
		else
		{
			UE_LOGF(LogICVFXTest, Warning, "%ls: interval is set to 0 and no timer is set, skipping...", ANSI_TO_TCHAR(__func__));
		}
	}
}

void UICVFXTestControllerBase::ClearMemReportTimer()
{
	if (UWorld* const World = GetWorld())
	{
		if (MemReportTimerHandle.IsValid())
		{
			World->GetTimerManager().ClearTimer(MemReportTimerHandle);
			UE_LOGF(LogICVFXTest, Display, "%ls: cleared...", ANSI_TO_TCHAR(__func__));
		}
	}
}

bool UICVFXTestControllerBase::TryStartingVideoCapture()
{
	if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
	{
		const EVideoRecordingState RecordingState = VideoRecordingSystem->GetRecordingState();

		if (RecordingState == EVideoRecordingState::None)
		{
			VideoRecordingSystem->EnableRecording(true);

			const FString Suffix = FString::Printf(TEXT("_%s"), *FDateTime::Now().ToString(TEXT("%H-%M-%S")));
			const FString VideoFilename = FString::Printf(TEXT("AutoTest_CL-%d%s"), FEngineVersion::Current().GetChangelist(), *Suffix);
			VideoRecordingTitle = FText::FromString(VideoFilename);
			constexpr bool bInAutoStart = true;
			constexpr bool bInAutoContinue = false;
			constexpr bool bInExportToLibrary = false;
			FVideoRecordingParameters VideoRecordingParameters(VideoRecordingSystem->GetMaximumRecordingSeconds(), bInAutoStart, bInAutoContinue, bInExportToLibrary, FPlatformMisc::GetPlatformUserForUserIndex(0));
			VideoRecordingSystem->NewRecording(*VideoFilename, VideoRecordingParameters);

			if (VideoRecordingSystem->IsEnabled())
			{
				if (RecordingState == EVideoRecordingState::Starting || RecordingState == EVideoRecordingState::Recording)
				{
					UE_LOGF(LogICVFXTest, Log, "AutoTest %ls: starting video recording %ls...", ANSI_TO_TCHAR(__func__), *VideoFilename);
					return true;
				}
				else
				{
					UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: failed to start video recording %ls.", ANSI_TO_TCHAR(__func__), *VideoFilename);
				}
			}
			else
			{
				UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: video recording could not be enabled.", ANSI_TO_TCHAR(__func__));
			}
		}
		else
		{
			UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: could not start a new recording, may be already recording.", ANSI_TO_TCHAR(__func__));
		}
	}
	else
	{
		UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: video recording system is null.", ANSI_TO_TCHAR(__func__));
	}

	return false;
}

bool UICVFXTestControllerBase::TryFinalizingVideoCapture(const bool bStopAutoContinue/*=false*/)
{
	if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
	{
		if (VideoRecordingSystem->GetRecordingState() != EVideoRecordingState::None)
		{
			VideoRecordingSystem->FinalizeRecording(true, VideoRecordingTitle, FText::GetEmpty(), bStopAutoContinue);

			if (VideoRecordingSystem->GetRecordingState() == EVideoRecordingState::Finalizing)
			{
				UE_LOGF(LogICVFXTest, Log, "AutoTest %ls: finalizing recording...", ANSI_TO_TCHAR(__func__));
				return true;
			}
		}
	}
	else
	{
		UE_LOGF(LogICVFXTest, Warning, "AutoTest %ls: video recording system is null.", ANSI_TO_TCHAR(__func__));
	}

	return false;
}

void UICVFXTestControllerBase::OnPreWorldInitializeInternal(UWorld* World, const UWorld::InitializationValues IVS)
{
	TryEarlyExec(World);
	OnPreWorldInitialize(World);
}

void UICVFXTestControllerBase::TryEarlyExec(UWorld* const World)
{
	check(World);

	if (GEngine)
	{
		// Search the list of deferred commands
		const TArray<FString>& DeferredCmds = GEngine->DeferredCommands;
		TArray<int32> ExecutedIndices;
		for (int32 DeferredCmdIndex = 0; DeferredCmdIndex < DeferredCmds.Num(); ++DeferredCmdIndex)
		{
			// If the deferred command is one that should be executed early
			const FString& DeferredCmd = DeferredCmds[DeferredCmdIndex];
			if (CmdsToExecEarly.ContainsByPredicate([&DeferredCmd](const FString& CmdToFind) { return DeferredCmd.StartsWith(CmdToFind); }))
			{
				UE_LOGF(LogICVFXTest, Display, "%ls: executing '%ls' early.", ANSI_TO_TCHAR(__func__), *DeferredCmd);
				GEngine->Exec(World, *DeferredCmd);
				ExecutedIndices.Push(DeferredCmdIndex);
			}
		}

		// Remove the executed commands from the list of deferred commands
		// Note: This is done in reverse order to ensure the cached indices remain valid
		while (!ExecutedIndices.IsEmpty())
		{
			GEngine->DeferredCommands.RemoveAt(ExecutedIndices.Pop());
		}
	}
}

void UICVFXTestControllerBase::OnMemReportTimerExpired()
{
	ExecuteMemReport();
}

void UICVFXTestControllerBase::OnMemReportIntervalChanged()
{
	if (MemReportTimerHandle.IsValid())
	{
		if (UWorld* const World = GetWorld())
		{
			if (World->GetTimerManager().GetTimerRate(MemReportTimerHandle) != ICVFXTest::CVarMemReportInterval->GetFloat())
			{
				SetMemReportTimer();
			}
		}
	}
}
