// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ICVFXTestControllerBase.h"

#include "Cluster/IDisplayClusterClusterManager.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Controllers/LiveLinkTransformController.h"
#include "DisplayClusterRootActor.h"
#include "Game/IDisplayClusterGameManager.h"
#include "HAL/IConsoleManager.h"
#include "IDisplayClusterCallbacks.h"
#include "ICVFXTestLocation.h"
#include "IDisplayCluster.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkPreset.h"
#include "LiveLinkPresetTypes.h"
#include "Roles/LiveLinkTransformRole.h"

#include "ICVFXTestControllerAutoTest.generated.h"

class ULocalPlayer;

UENUM()
enum class EICVFXAutoTestState : uint8
{
	InitialLoad,
	Soak,
	TraverseTestLocations,
	Finished,
	Shutdown,

	MAX
};

class FICVFXAutoTestState
{
public:
	using Super = FICVFXAutoTestState;

	FICVFXAutoTestState() = delete;
	FICVFXAutoTestState(class UICVFXTestControllerAutoTest* const TestController) : Controller(TestController) {}
	virtual ~FICVFXAutoTestState() {}

	virtual void Start(const EICVFXAutoTestState PrevState);
	virtual void End(const EICVFXAutoTestState NewState) {}
	virtual void Tick(const float TimeDelta);

	double GetTestStateTime() const 
	{
		return TimeSinceStart;
	}

protected:
	class UICVFXTestControllerAutoTest* Controller;

private:
	double TimeSinceStart = 0.0;
};

UCLASS()
class UICVFXTestControllerAutoTest : public UICVFXTestControllerBase
{
	GENERATED_BODY()

public:
	FString GetStateName(const EICVFXAutoTestState State) const;
	FICVFXAutoTestState& GetTestState() const;
	FICVFXAutoTestState& SetTestState(const EICVFXAutoTestState NewState);
	void SetTestLocations(const TArray<AActor*> TestLocations);
	virtual void EndICVFXTest(const int32 ExitCode=0) override;

	double GetCurrentStateTime() const
	{
		return GetTimeInCurrentState();
	}

	int32 GetCurrentTestLocationIndex() const
	{
		return CurrentTestLocationIndex;
	}

	void GoToTestLocation(int32 Index);

	void GoToNextTestLocation()
	{
		CurrentTestLocationIndex++;
		GoToTestLocation(CurrentTestLocationIndex);
	}

	int32 NumTestLocations() const
	{
		return TestLocations.Num();
	}

	void SetInnerGPUIndex(int32 InGPUIndex)
	{
		InnerGPUIndex = InGPUIndex;
	}

	int32 GetInnerGPUIndex() const
	{
		return InnerGPUIndex.load();
	}

	void InitializeLiveLink();
	void UpdateInnerGPUIndex();
	void UpdateTestLocations();

protected:
	virtual void OnInit() override;
	virtual void OnPreMapChange() override;
	virtual void OnTick(float TimeDelta) override;
	virtual void BeginDestroy() override;

public:
	float TimePerTestLocation = 60.f;
	double TimeAtTestLocation = 0.0;
	TObjectPtr<AActor> DisplayClusterActor = nullptr;

private:
	virtual void UnbindAllDelegates() override;

	FICVFXAutoTestState* States[(uint8)EICVFXAutoTestState::MAX];
	float StateTimeouts[(uint8)EICVFXAutoTestState::MAX];
	EICVFXAutoTestState CurrentState;

	virtual void OnPreWorldInitialize(UWorld* World) override;

	UFUNCTION()
	void OnWorldBeginPlay();

	UFUNCTION()
	void OnGameStateSet(AGameStateBase* const GameStateBase);
	
	FConsoleVariableSinkHandle SoakTimeSink;

	UFUNCTION()
	void OnSoakTimeChanged();

	TArray<AActor*> TestLocations;

	int32 CurrentTestLocationIndex = 0;

	/** What GPU Index should be used for the inner viewport. */
	std::atomic<int32> InnerGPUIndex = 0;
};
