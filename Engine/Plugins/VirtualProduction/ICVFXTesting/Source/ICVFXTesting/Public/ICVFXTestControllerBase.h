// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GauntletTestController.h"

#include "Engine/TimerHandle.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "TimerManager.h"

#include "ICVFXTestControllerBase.generated.h"


DECLARE_LOG_CATEGORY_EXTERN(LogICVFXTest, Log, All)

class AActor;
class AGameState;
class IConsoleVariable;

UCLASS()
class UICVFXTestControllerBase : public UGauntletTestController
{
	GENERATED_BODY()

public:
	UICVFXTestControllerBase(const FObjectInitializer& ObjectInitializer);

protected:
	// ~Begin UGauntletTestController Interface
	virtual void OnInit() override;
	virtual void OnTick(float TimeDelta) override;
	virtual void OnStateChange(FName OldState, FName NewState) override;
	virtual void BeginDestroy() override;
	// ~End UGauntletTestController Interface

	virtual void EndICVFXTest(const int32 ExitCode=0);
	virtual void UnbindAllDelegates();

public:
	uint32 GetRunCount() const;
	uint32 GetMaxRunCount() const;
	uint32 GetRunsRemaining() const;
	uint32 MarkRunComplete();

	bool RequestsFPSChart();
	bool RequestsMemReport();
	bool RequestsVideoCapture();

	void ConsoleCommand(const TCHAR* Cmd);

	void ExecuteMemReport(const TOptional<FString> Args=TOptional<FString>());
	void SetMemReportTimer(const TOptional<float> Interval=TOptional<float>());
	void ClearMemReportTimer();

	bool TryStartingVideoCapture();
	bool TryFinalizingVideoCapture(const bool bStopAutoContinue = false);

public:
	FString DisplayClusterUObjectPath;

private:
	float WarnStuckTime;
	float NextWarnStuckTime;

	void OnPreWorldInitializeInternal(UWorld* World, const UWorld::InitializationValues IVS);
	virtual void OnPreWorldInitialize(UWorld* World) {}

	UFUNCTION()
	void TryEarlyExec(UWorld* const World);

	/** List of command prefixes used to find and execute commands passed in through commandline or config OnInit. */
	const TArray<FString> CmdsToExecEarly = {
		TEXT("t.FPSChart.DoCsvProfile"),
		TEXT("ICVFXTest"),
		TEXT("r.nanite"),
		TEXT("r.ScreenPercentage"),
		TEXT("r.RayTracing"),
		TEXT("r.DynamicGlobalIlluminationMethod"),
		TEXT("r.ReflectionMethod"),
		TEXT("r.Lumen"),
		TEXT("FX.AllowGPUParticles"),
		TEXT("r.Shadow.Virtual.Enable")};

	uint32 RunCount;

	bool bRequestsFPSChart;
	bool bRequestsMemReport;
	bool bRequestsVideoCapture;

	FTimerHandle MemReportTimerHandle;
	FTimerDelegate MemReportTimerDelegate;
	FConsoleVariableSinkHandle MemReportIntervalSink;

	UFUNCTION()
	void OnMemReportTimerExpired();

	UFUNCTION()
	void OnMemReportIntervalChanged();

	FText VideoRecordingTitle;
};