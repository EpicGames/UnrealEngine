// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraActor.h"

#include "ICVFXTestLocation.generated.h"

/**
 * Actor used when running ICVFX perf tests. For every test location, the display cluster root actor will be moved to this location and perf data will be connected for a minute.
 */
UCLASS(Blueprintable, BlueprintType, Category="ICVFX", DisplayName="ICVFXTestLocation")
class ICVFXTESTING_API AICVFXTestLocation : public ACameraActor
{
public:
    GENERATED_BODY()
    
};